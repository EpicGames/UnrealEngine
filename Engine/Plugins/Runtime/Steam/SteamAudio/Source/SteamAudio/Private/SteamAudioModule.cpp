//
// Copyright (C) Valve Corporation. All rights reserved.
//

#include "SteamAudioModule.h"
#include "Misc/Paths.h"
#include "Features/IModularFeatures.h"

DEFINE_LOG_CATEGORY(LogSteamAudio);

IMPLEMENT_MODULE(SteamAudio::FSteamAudioModule, SteamAudio)

namespace SteamAudio
{
	static bool bModuleStartedUp = false;

	FSteamAudioModule::FSteamAudioModule()
	{
	}

	FSteamAudioModule::~FSteamAudioModule()
	{
	}

	void FSteamAudioModule::StartupModule()
	{
		check(bModuleStartedUp == false);

		bModuleStartedUp = true;

		UE_LOGF(LogSteamAudio, Log, "FSteamAudioModule Startup");

		UE_LOGF(LogSteamAudio, Warning, "The Steam Audio plugin is deprecated and will be removed in a future engine release. Get Valve's version here: https://valvesoftware.github.io/steam-audio/doc/unreal/getting-started.html")
	}

	void FSteamAudioModule::ShutdownModule()
	{
		UE_LOGF(LogSteamAudio, Log, "FSteamAudioModule Shutdown");

		check(bModuleStartedUp == true);

		bModuleStartedUp = false;
	}

}

