//
// Copyright (C) Valve Corporation. All rights reserved.
//

#pragma once

#include "ISteamAudioModule.h"
#include "AudioPluginUtilities.h"
#include "AudioDevice.h"

DECLARE_LOG_CATEGORY_EXTERN(LogSteamAudio, Log, All);

namespace SteamAudio
{
	/************************************************************************/
	/* FSteamAudioModule                                                    */
	/* Module interface for Steam Audio. Registers the Plugin Factories     */
	/* and handles the third party DLL.                                     */
	/************************************************************************/
	class FSteamAudioModule : public ISteamAudioModule
	{
	public:
		FSteamAudioModule();
		~FSteamAudioModule();

		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
		//~ End IAudioPlugin
	};
}