//
// Copyright (C) Valve Corporation. All rights reserved.
//

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class SteamAudio : ModuleRules
	{
		public SteamAudio(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"AudioMixer",
					"InputCore",
					"RenderCore",
					"RHI"
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"TargetPlatform",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"InputCore",
					"Projects",
					"AudioMixer",
					"AudioExtensions"
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("EditorFramework");
				PrivateDependencyModuleNames.Add("UnrealEd");
				PrivateDependencyModuleNames.Add("Landscape");
            }
            else
            {
                RuntimeDependencies.Add("$(ProjectDir)/Content/SteamAudio/Runtime/...");
            }

			AddEngineThirdPartyPrivateStaticDependencies(Target, "libPhonon");

			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11Audio");

                RuntimeDependencies.Add("$(EngineDir)/Binaries/ThirdParty/Phonon/Win64/...");
                PublicDelayLoadDLLs.Add("GPUUtilities.dll");
				PublicDelayLoadDLLs.Add("tanrt64.dll");
			}
			else if (Target.Platform == UnrealTargetPlatform.Android)
			{
				string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
				AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "SteamAudio_APL.xml"));
			}
		}
	}
}
