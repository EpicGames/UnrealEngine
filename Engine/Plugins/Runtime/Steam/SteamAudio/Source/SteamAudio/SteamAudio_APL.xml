<?xml version="1.0" encoding="utf-8"?>
<!--Steam Audio plugin additions-->
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<!-- init section is always evaluated once per architecture -->
	<init>
		<log text="Steam Audio init"/>
		<setBool result="bSupported" value="false"/>
		<isArch arch="armeabi-v7a">
			<setBool result="bSupported" value="true"/>
		</isArch>
	</init>

	<!-- optional files or directories to copy to Intermediate/Android/APK -->
	<resourceCopies>
		<isArch arch="armeabi-v7a">
			<copyFile src="$S(EngineDir)/Source/ThirdParty/libPhonon/phonon_api/lib/Android/libphonon.so"
						dst="$S(BuildDir)/libs/armeabi-v7a/libphonon.so" />
      <log text="Copying libphonon.so"/>
		</isArch>
	</resourceCopies>

	<!-- optional libraries to load in GameActivity.java before libUnreal.so -->
	<soLoadLibrary>
		<if condition="bSupported">
			<true>
				<loadLibrary name="phonon" failmsg="Steam Audio library not loaded and required!" />
			</true>
		</if>
	</soLoadLibrary>
</root>
