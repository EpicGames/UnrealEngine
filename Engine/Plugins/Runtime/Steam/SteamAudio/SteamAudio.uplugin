{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "2.0-beta.17",
	"FriendlyName" : "Steam Audio (Deprecated)",
	"Description" : "This plugin is deprecated and will be removed in a future engine release. Please use the plugin from Valve's website.",
	"Category" : "Audio",
	"CreatedBy" : "Valve Corporation",
	"CreatedByURL" : "https://valvesoftware.github.io/steam-audio",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "https://steamcommunity.com/app/596420",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "SteamAudio",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault",
			"PlatformAllowList" :
			[
				"Win64", "Android"
			],
			"PlatformArchitectureDenyList":
			[
				"Win64:arm64"
			]
		}
	]
}
